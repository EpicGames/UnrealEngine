{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Control Rig Spline",
	"Description": "Allows creation and use of splines for Control Rig",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion":  false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "ControlRigSpline",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}