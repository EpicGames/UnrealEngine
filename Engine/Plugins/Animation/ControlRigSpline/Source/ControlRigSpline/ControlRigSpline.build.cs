// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ControlRigSpline : ModuleRules
	{
		public ControlRigSpline(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RigVM",
				"ControlRig",
			});
		}
	}
}
