// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSplineModule.h"

void FControlRigSplineModule::StartupModule()
{

}

void FControlRigSplineModule::ShutdownModule()
{

}

IMPLEMENT_MODULE(FControlRigSplineModule, ControlRigSpline)
