// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSplineTypes.h"

#include "RigVMFunctions/Math/RigVMMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSplineTypes)

ControlRigBaseSpline::ControlRigBaseSpline(const TArrayView<const FVector>& InControlPoints, const uint16 InDegree, bool bInClosed)
: Degree(InDegree)
, bClosed(bInClosed)
{
	if (bClosed)
	{
		int32 NumPoints = InControlPoints.Num() + Degree;
		ControlPoints.Reserve(NumPoints);
		
		int32 Index = InControlPoints.Num() - FMath::RoundToPositiveInfinity((float)Degree/2);
		while (ControlPoints.Num() < NumPoints)
		{
			Index = (Index+1)%InControlPoints.Num();
			ControlPoints.Add(InControlPoints[Index]);
		}
	}
	else
	{
		ControlPoints = InControlPoints;
	}
}

TArray<FVector> ControlRigBaseSpline::GetControlPointsWithoutDuplicates()
{
	if (bClosed)
	{
		const int32 NumPoints = ControlPoints.Num() - Degree;
		const int32 NumInitPointsToIgnore = FMath::RoundToPositiveInfinity((float)Degree/2) - 1;
		
		TArray<FVector> ReducedControlPoints;
		ReducedControlPoints.Reserve(NumPoints);
		for (int32 i=0; i<NumPoints; ++i)
		{
			ReducedControlPoints.Add(ControlPoints[i+NumInitPointsToIgnore]);
		}
		return ReducedControlPoints;
	}

	return ControlPoints;
}

TArray<uint16> ControlRigBaseSpline::GetControlIndicesWithoutDuplicates()
{
	if (bClosed)
	{
		const int32 NumPoints = ControlPoints.Num() - Degree;
		const int32 NumInitPointsToIgnore = FMath::RoundToPositiveInfinity((float)Degree/2) - 1;
		
		TArray<uint16> ReducedControlIndices;
		ReducedControlIndices.Reserve(NumPoints);
		for (int32 i=0; i<NumPoints; ++i)
		{
			ReducedControlIndices.Add(i+NumInitPointsToIgnore);
		}
		return ReducedControlIndices;
	}

	TArray<uint16> ControlIndices;
	ControlIndices.SetNumUninitialized(ControlPoints.Num());
	for(int32 i = 0; i<ControlPoints.Num(); ++i)
	{
		ControlIndices[i] = i;
	}
	return ControlIndices;
}

void ControlRigBaseSpline::SetControlPoints(const TArrayView<const FVector>& InControlPoints)
{
	if (bClosed)
	{
		int32 CurIndex = 0;
		int32 Index = InControlPoints.Num() - FMath::RoundToPositiveInfinity((float)Degree/2);
		while (CurIndex < ControlPoints.Num())
		{
			Index = (Index+1)%InControlPoints.Num();
			ControlPoints[CurIndex++] = InControlPoints[Index];
		}
	}
	else
	{
		ControlPoints = InControlPoints;
	}
}

ControlRigBSpline::ControlRigBSpline(const TArrayView<const FVector>& InControlPoints, const uint16 InDegree, const bool bInClosed, bool bInClamped)
: ControlRigBaseSpline(InControlPoints, InDegree, bInClosed)
{
	const uint16 N = ControlPoints.Num() - 1;
	const uint16 P = Degree;
	const uint16 M = N + P + 1;
	KnotVector.SetNumZeroed(M+1);

	// Clamped means there is p+1 0s at the beginning and 1s at the end of the knot vector
	if (bInClamped && !bInClosed)
	{
		for (int32 i=M-P; i<=M; ++i)
		{
			KnotVector[i] = 1.f;
		}

		float Dk = 1.f/(M-P - (P+1) +1);
		for (int32 i=P+1; i<M-P; ++i)
		{
			KnotVector[i] = (i-P)*Dk;
		}
	}
	else
	{
		float Dk = 1.f / M;
		for (int32 i = 0; i<M+1; ++i)
		{
			KnotVector[i] = i*Dk;
		}
	}
}

FVector ControlRigBSpline::GetPointAtParam(float Param)
{
	// DeBoor's Algorithm
	// https://pages.mtu.edu/~shene/COURSES/cs3621/NOTES/spline/B-spline/de-Boor.html
	
	const float ClampedU = FMath::Clamp(Param, 0.f, 1.f);

	const uint16 N = ControlPoints.Num() - 1;
	const uint16 P = Degree;
	const uint16 M = N + P + 1;

	// Remap U value to KnotVector[P] : KnotVector[M+1 - (P+1)]
	const float Min = KnotVector[P] + KINDA_SMALL_NUMBER;
	const float Max = KnotVector[M-P] - KINDA_SMALL_NUMBER;
	const float U = Min + ClampedU*(Max-Min);

	// Find [uk, uk+1) where U lives (or k == the first appearence of U)
	uint16 K = 0;
	{
		// Find [uk, uk+1) where U lives
		for (uint16 i=0; i<KnotVector.Num()-1; ++i)
		{
			if ((FMath::IsNearlyEqual(KnotVector[i], U)) ||
				(KnotVector[i] < U && KnotVector[i+1] > U))
			{
				K = i;
				break;
			}
		}
	}

	TArray<FVector> WorkingPoints;
	WorkingPoints.SetNum(P+1);
	for (int32 i=K-P; i<=K; ++i)
	{
		WorkingPoints[i-(K-P)] = ControlPoints[i];
	}
	
	for (uint16 R = 1; R <= Degree; ++R)
	{
		uint16 StartCPIndex = K-P+R;
		uint16 EndCPIndex = K;
		if (FMath::IsNearlyEqual(KnotVector[StartCPIndex], U))
		{
			continue;
		}

		uint16 Index = 0;
		for (uint16 i = StartCPIndex; i <= EndCPIndex; ++i, ++Index)
		{
			float Air = (U - KnotVector[i]) / (KnotVector[i+P-R+1] - KnotVector[i]);
			WorkingPoints[Index] = (1-Air) * WorkingPoints[Index] + Air * WorkingPoints[Index+1];
		}
	}

	return WorkingPoints[0];
}

ControlRigHermite::ControlRigHermite(const TArrayView<const FVector>& InControlPoints, const bool bInClosed)
: ControlRigBaseSpline(InControlPoints, 4, bInClosed)
{
	if (bClosed)
	{
		SegmentPoints.Append(ControlPoints);
		NumSegments = SegmentPoints.Num() - 4;
	}
	else
	{
		SegmentPoints.Reserve(ControlPoints.Num()+2);
		SegmentPoints.Add(ControlPoints[0] + ControlPoints[0] - ControlPoints[1]);
		SegmentPoints.Append(ControlPoints);
		SegmentPoints.Add(ControlPoints.Last() + ControlPoints.Last() - ControlPoints[ControlPoints.Num()-2]);
		NumSegments = SegmentPoints.Num() - 3;
	}
}

void ControlRigHermite::SetControlPoints(const TArrayView<const FVector>& InControlPoints)
{
	ControlRigBaseSpline::SetControlPoints(InControlPoints);

	SegmentPoints.Reset();
	if (bClosed)
	{
		SegmentPoints.Append(ControlPoints);
	}
	else
	{
		SegmentPoints.Reserve(ControlPoints.Num()+2);
		SegmentPoints.Add(ControlPoints[0] + ControlPoints[0] - ControlPoints[1]);
		SegmentPoints.Append(ControlPoints);
		SegmentPoints.Add(ControlPoints.Last() + ControlPoints.Last() - ControlPoints[ControlPoints.Num()-2]);
	}
}

static FVector Hermite(const FVector& P0, const FVector& P1, const FVector& M0, const FVector& M1, float Param)
{
	// https://en.wikipedia.org/wiki/Cubic_Hermite_spline#Catmull–Rom_spline

	const float T = Param;
	const float T2 = T*T;
	const float T3 = T2*T;
	
	const float H00 = 2*T3 - 3*T2 + 1;
	const float H10 = T3 - 2*T2 + T;
	const float H01 = -2*T3 + 3*T2;
	const float H11 = T3 - T2;

	return H00*P0 + H10*M0 + H01*P1 + H11*M1;
}

FVector ControlRigHermite::GetPointAtParam(float Param)
{
	const float ClampedParam = FMath::Clamp(Param, 0.f, 1.f);	

	const float ParamIntervals = 1.f / (float) NumSegments;
	uint16 SegmentIndex = FMath::Floor(ClampedParam / ParamIntervals);
	if (SegmentIndex >= NumSegments)
	{
		SegmentIndex = NumSegments-1;
	}

	uint16 P3Index = SegmentIndex + 3;
	P3Index = (P3Index < SegmentPoints.Num()) ? P3Index : SegmentPoints.Num()-1;
	const uint16 P0Index = P3Index - 3;
	const uint16 P1Index = P3Index - 2;
	const uint16 P2Index = P3Index - 1;

	const float StartSegmentParam = SegmentIndex * ParamIntervals;
	const float T = FMath::Clamp((ClampedParam-StartSegmentParam)/ParamIntervals, 0.f, 1.f);

	const FVector& P0 = SegmentPoints[P0Index];
	const FVector& P1 = SegmentPoints[P1Index];
	const FVector& P2 = SegmentPoints[P2Index];
	const FVector& P3 = SegmentPoints[P3Index];

	// https://www.cs.cmu.edu/~fp/courses/graphics/asst5/catmullRom.pdf
	static const float Tension = 0.5f;
	const FVector M1 = Tension * (P2 - P0);
	const FVector M2 = Tension * (P3 - P1);
	return Hermite(P1, P2, M1, M2, T);
}

FControlRigSplineImpl::~FControlRigSplineImpl()
{
	if (Spline)
	{
		delete Spline;
		Spline = nullptr;
	}
}

TArray<FVector>& FControlRigSplineImpl::GetControlPoints()
{
	check(Spline);
	return Spline->GetControlPoints();
}

TArray<FTransform>& FControlRigSplineImpl::GetControlTransforms()
{
	return ControlTransforms;
}

TArray<FVector> FControlRigSplineImpl::GetControlPointsWithoutDuplicates()
{
	check(Spline);
	return Spline->GetControlPointsWithoutDuplicates();
}

TArray<FTransform> FControlRigSplineImpl::GetControlTransformsWithoutDuplicates()
{
	check(Spline);
	TArray<uint16> ControlIndices = Spline->GetControlIndicesWithoutDuplicates();
	TArray<FTransform> Transforms;
	Transforms.Reserve(ControlIndices.Num());
	for (const uint16& Index : ControlIndices)
	{
		int32 WrappedIndex = int32(Index);

		// for closed splines we may ask for a control index above the index count,
		// so we need to wrap around along the spline and start over from the front.
		if(!ControlTransforms.IsValidIndex(WrappedIndex))
		{
			WrappedIndex = WrappedIndex % ControlTransforms.Num();
		}
		Transforms.Add(ControlTransforms[WrappedIndex]);
	}
	return Transforms;
}

uint8 FControlRigSplineImpl::GetDegree() const
{
	check(Spline);
	return Spline->GetDegree();
}

uint16 FControlRigSplineImpl::NumSamples() const
{
	return SamplesArray.Num();
}

FControlRigSpline::FControlRigSpline(const FControlRigSpline& InOther)
{
	SplineData = InOther.SplineData;
}

FControlRigSpline& FControlRigSpline::operator=(const FControlRigSpline& InOther)
{
	SplineData = InOther.SplineData;
	return *this;
}

uint8 FControlRigSpline::GetDegree() const
{
	if (!SplineData.IsValid())
	{
		return 0;
	}
	return SplineData->GetDegree();
}

void FControlRigSpline::SetControlPoints(const TArrayView<const FVector>& InPoints, const ESplineType SplineMode, const bool bInClosed, const int32 SamplesPerSegment, const float Compression, const float Stretch)
{
	TArray<FTransform> Transforms;
	Transforms.Reserve(InPoints.Num());
	for (const FVector& Point : InPoints)
	{
		FTransform Transform = FTransform::Identity;
		Transform.SetTranslation(Point);
		Transforms.Add(Transform);
	}
	return SetControlTransforms(Transforms, SplineMode, bInClosed, SamplesPerSegment, Compression, Stretch);
}

void FControlRigSpline::SetControlTransforms(const TArrayView<const FTransform>& InTransforms, const ESplineType SplineMode, const bool bInClosed, const int32 SamplesPerSegment, const float Compression, const float Stretch)
{
	const int32 ControlPointsCount = InTransforms.Num();
	if (ControlPointsCount < 4)
	{
		return;
	}

	if (SamplesPerSegment < 1)
	{
		return;
	}

	if (!SplineData.IsValid())
	{
		SplineData = MakeShared<FControlRigSplineImpl>();
	}

	TArray<FTransform> OldControlTransforms;
	if (SplineData->Spline)
	{
		OldControlTransforms = SplineData->GetControlTransformsWithoutDuplicates();
	}

	bool bControlPointsChanged = false;
	const bool bNumControlPointsChanged = (SplineData->Spline) ? OldControlTransforms.Num() != ControlPointsCount : true;
	if (!bNumControlPointsChanged)
	{
		for (int32 i=0; i<ControlPointsCount; ++i)
		{
			if (!InTransforms[i].Equals(OldControlTransforms[i]))
			{
				bControlPointsChanged = true;
				break;
			}
		}
	}
	else
	{
		bControlPointsChanged = true;
	}
	
	const bool bSplineModeChanged = SplineMode != SplineData->SplineMode;
	const bool bSamplesCountChanged = SamplesPerSegment != SplineData->SamplesPerSegment;
	const bool bStretchChanged = Stretch != SplineData->Stretch || Compression != SplineData->Compression;
	const bool bClosedChanged = bInClosed != SplineData->bClosed;
	if (!bSplineModeChanged && !bControlPointsChanged && !bSamplesCountChanged && !bStretchChanged && !bClosedChanged)
	{
		return;
	}

	SplineData->ControlTransforms = InTransforms;
	SplineData->SplineMode = SplineMode;
	SplineData->bClosed = bInClosed;
	SplineData->SamplesPerSegment = SamplesPerSegment;
	SplineData->Compression = Compression;
	SplineData->Stretch = Stretch;

	// If we need to update the spline because the controls points have changed, or the spline mode has changed
	if (bControlPointsChanged || bSplineModeChanged || bClosedChanged)
	{
		TArray<FVector> ControlPoints;
		ControlPoints.Reserve(InTransforms.Num());
		for (const FTransform& Transform : InTransforms)
		{
			ControlPoints.Add(Transform.GetTranslation());
		}
		
		if (bNumControlPointsChanged || bSplineModeChanged || bClosedChanged)
		{
			// Delete previously created spline
			if (SplineData->Spline)
			{
				delete SplineData->Spline;
				SplineData->Spline = nullptr;
			}
			
			switch (SplineMode)
			{
				case ESplineType::BSpline:
				{
					SplineData->Spline = new ControlRigBSpline(ControlPoints, 3, bInClosed, true);
					break;
				}
				case ESplineType::Hermite:
				{
					
					SplineData->Spline = new ControlRigHermite(ControlPoints, bInClosed);
					break;
				}
				default:
				{
					checkNoEntry(); // Unknown Spline Mode
					break;
				}
			}
		}
		else if (bControlPointsChanged)
		{
			SplineData->Spline->SetControlPoints(ControlPoints);
		}
	}

	// If curve has changed, or sample count has changed, recompute the cache
	if (bControlPointsChanged || bSplineModeChanged || bSamplesCountChanged || bStretchChanged || bClosedChanged)
	{
		// Cache sample positions of the spline
		const int32 NumSamples = (ControlPointsCount-1) * SamplesPerSegment;
		SplineData->SamplesArray.SetNumUninitialized(NumSamples, EAllowShrinking::No);

		// Compute sample positions
		float U = 0.f;
		const float DeltaU = 1.f/(NumSamples-1);
		for (int32 i=0; i<NumSamples; ++i, U+=DeltaU)
		{
			SplineData->SamplesArray[i].SetTranslation(SplineData->Spline->GetPointAtParam(U));
		}

		// Interpolate sample rotations and scales
		U = 0.f;
		for (int32 i=0; i<NumSamples; ++i, U+=DeltaU)
		{
			// This tangent will always be the X axis of the sample transform
			FVector Tangent = TangentAtParam(U);
			Tangent.Normalize();

			FVector UpVector = FVector::UnitY();
			FVector Binormal = Tangent.Cross(UpVector);
			Binormal.Normalize();
			UpVector = Binormal.Cross(Tangent);
			UpVector.Normalize();
			
			FMatrix Matrix(Tangent, UpVector, Binormal, FVector::ZeroVector);
			SplineData->SamplesArray[i].SetRotation(Matrix.ToQuat());
			SplineData->SamplesArray[i].SetScale3D(FVector(1.f,1.f,1.f));

			FTransform& ParallelTransform = SplineData->SamplesArray[i];
			
			int32 PrevIndex = U*(ControlPointsCount-1);
			int32 NextIndex = PrevIndex+1;
			if (!InTransforms.IsValidIndex(NextIndex))
			{
				PrevIndex--;
				NextIndex--;
			}

			// Interpolate inside segment
			const float UPrev = PrevIndex / (float)(ControlPointsCount-1);
			const float UNext = NextIndex / (float)(ControlPointsCount-1);
			const float Interp = (U - UPrev) / (UNext - UPrev);

			const FTransform TransportedTransformPrev = ParallelTransform * InTransforms[PrevIndex];
			const FTransform TransportedTransformNext = ParallelTransform * InTransforms[NextIndex];
			const FQuat DiffRotation = FQuat::Slerp(InTransforms[PrevIndex].GetRotation(), InTransforms[NextIndex].GetRotation(), Interp);

			FVector UpVectorRotated = DiffRotation.RotateVector(FVector::UnitY());
			FVector BinormalRotated = Tangent.Cross(UpVectorRotated);
			BinormalRotated.Normalize();
			UpVectorRotated = BinormalRotated.Cross(Tangent);
			UpVectorRotated.Normalize();
			FMatrix MatrixRotated(Tangent, UpVectorRotated, BinormalRotated, FVector::ZeroVector);
			SplineData->SamplesArray[i].SetRotation(MatrixRotated.ToQuat());
			SplineData->SamplesArray[i].SetScale3D(FVector(1.f,1.f,1.f));
		}

		// Correct length of samples
		if (!bSplineModeChanged && !bSamplesCountChanged && !bNumControlPointsChanged && !bClosedChanged)
		{
			if (SplineData->InitialLengths.Num() > 0)
			{
				const TArray<FTransform> SamplesBeforeCorrect = SplineData->SamplesArray;
				for (int32 i = 0; i < ControlPointsCount - 1; ++i)
				{
					for (int32 j = (i == 0) ? 1 : 0; j < SamplesPerSegment; ++j)
					{
						const int32 CurSampleIndex = i * SamplesPerSegment + j;
						const int32 PrevSampleIndex = CurSampleIndex - 1;
						if (!SplineData->InitialLengths.IsValidIndex(PrevSampleIndex))
						{
							break;
						}
						
						// Get direction from samples before correction
						FVector Dir = SamplesBeforeCorrect[CurSampleIndex].GetTranslation() - SamplesBeforeCorrect[PrevSampleIndex].GetTranslation();
						Dir.Normalize();

						float InitialLength = SplineData->InitialLengths[PrevSampleIndex];
						// Current length as the projection on the Dir vector (might be negative)
						float CurrentLength = (SplineData->SamplesArray[CurSampleIndex].GetTranslation() - SplineData->SamplesArray[PrevSampleIndex].GetTranslation()).Dot(Dir);
						float FixedLength = FMath::Clamp(CurrentLength,
							(Compression > 0.f) ? InitialLength * Compression : CurrentLength,
							(Stretch > 0.f) ? InitialLength * Stretch : CurrentLength);

						SplineData->SamplesArray[CurSampleIndex].SetTranslation(SplineData->SamplesArray[PrevSampleIndex].GetTranslation() + Dir * FixedLength);
					}
				}
			}
		}

		// Cache accumulated length at sample array
		if (SplineData->SamplesArray.Num() > 0)
		{
			SplineData->AccumulatedLenth.SetNumUninitialized(SplineData->SamplesArray.Num(), EAllowShrinking::No);
			SplineData->AccumulatedLenth[0] = 0.f;
			if (bSplineModeChanged || bClosedChanged || bSamplesCountChanged || bNumControlPointsChanged)
			{
				SplineData->InitialLengths.SetNumUninitialized(SplineData->SamplesArray.Num() - 1);
			}
			for (int32 i = 1; i < SplineData->SamplesArray.Num(); ++i)
			{
				float CurrentLength = FVector::Distance(SplineData->SamplesArray[i - 1].GetTranslation(), SplineData->SamplesArray[i].GetTranslation());
				if (bSplineModeChanged || bClosedChanged || bSamplesCountChanged || bNumControlPointsChanged)
				{
					SplineData->InitialLengths[i-1] = CurrentLength;
				}
				SplineData->AccumulatedLenth[i] = SplineData->AccumulatedLenth[i - 1] + CurrentLength;
			}
		}
	}
}

FVector FControlRigSpline::PositionAtParam(const float InParam) const
{
	if (!SplineData.IsValid())
	{
		return FVector();
	}

	const float ClampedU = FMath::Clamp<float>(InParam, 0.f, 1.f);

	const int32 LastIndex = SplineData->SamplesArray.Num() - 1;
	const float fIndexPrev = ClampedU * LastIndex;	
	const int32 IndexPrev = FMath::Floor<int32>(fIndexPrev);
	const float ULocal = fIndexPrev - IndexPrev;
	const int32 IndexNext = (IndexPrev < LastIndex) ? IndexPrev + 1 : IndexPrev;

	return SplineData->SamplesArray[IndexPrev].GetTranslation() * (1.f - ULocal) + SplineData->SamplesArray[IndexNext].GetTranslation() * ULocal;
}

FTransform FControlRigSpline::TransformAtParam(const float InParam) const
{
	if (!SplineData.IsValid())
	{
		return FTransform();
	}

	const float ClampedU = FMath::Clamp<float>(InParam, 0.f, 1.f);

	const int32 LastIndex = SplineData->SamplesArray.Num() - 1;
	const float fIndexPrev = ClampedU * LastIndex;	
	const int32 IndexPrev = FMath::Floor<int32>(fIndexPrev);
	const float ULocal = fIndexPrev - IndexPrev;
	const int32 IndexNext = (IndexPrev < LastIndex) ? IndexPrev + 1 : IndexPrev;

	FTransform Result;
	FQuat Rotation = FQuat::Slerp(SplineData->SamplesArray[IndexPrev].GetRotation(), SplineData->SamplesArray[IndexNext].GetRotation(), ULocal);
	Rotation.Normalize();
	Result.SetRotation(Rotation);
	Result.LerpTranslationScale3D(SplineData->SamplesArray[IndexPrev], SplineData->SamplesArray[IndexNext], ScalarRegister(ULocal));
	return Result;
}

FVector FControlRigSpline::TangentAtParam(const float InParam) const
{
	if (!SplineData.IsValid())
	{
		return FVector();
	}

	const float ClampedU = FMath::Clamp<float>(InParam, 0.f, 1.f);

	const int32 LastIndex = SplineData->SamplesArray.Num() - 1;
	const float fIndexPrev = ClampedU * LastIndex;	
	int32 IndexPrev = FMath::Floor<int32>(fIndexPrev);
	const float ULocal = fIndexPrev - IndexPrev;
	int32 IndexNext = IndexPrev + 1;
	
	if (IndexNext > LastIndex)
	{
		IndexPrev--;
		IndexNext--;
	}
	
	const FVector TangentStart = SplineData->SamplesArray[IndexNext].GetTranslation() - SplineData->SamplesArray[IndexPrev].GetTranslation();
	if (IndexNext + 1 < SplineData->SamplesArray.Num())
	{
		const FVector TangentEnd = SplineData->SamplesArray[IndexNext+1].GetTranslation() - SplineData->SamplesArray[IndexNext].GetTranslation();

		const FQuat FullQuat = FRigVMMathLibrary::FindQuatBetweenVectors(TangentStart, TangentEnd);
		const FQuat PartialQuat = FQuat::Slerp(FQuat::Identity, FullQuat, ULocal);

		const FVector Result = PartialQuat.RotateVector(TangentStart);

		return Result;
	}
	
	return TangentStart;
}

float FControlRigSpline::LengthAtParam(const float InParam) const
{
	if (!SplineData.IsValid())
	{
		return 0.f;
	}

	const float ClampedU = FMath::Clamp<float>(InParam, 0.f, 1.f);

	const int32 LastIndex = SplineData->SamplesArray.Num() - 1;
	const float fIndexPrev = ClampedU * LastIndex;	
	const int32 IndexPrev = FMath::Floor<int32>(fIndexPrev);
	const float ULocal = fIndexPrev - IndexPrev;
	const int32 IndexNext = (IndexPrev < LastIndex) ? IndexPrev + 1 : IndexPrev;

	return SplineData->AccumulatedLenth[IndexPrev] * (1.f - ULocal) + SplineData->AccumulatedLenth[IndexNext] * ULocal;
}

