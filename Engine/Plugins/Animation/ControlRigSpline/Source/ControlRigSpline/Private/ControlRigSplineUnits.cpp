// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSplineUnits.h"
#include "Units/RigUnitContext.h"
#include "Features/IModularFeatures.h"
#include "Algo/BinarySearch.h"
#include "ControlRig.h"
#include "Units/Highlevel/Hierarchy/RigUnit_FitChainToCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSplineUnits)

FTransform ApplyPrimarySecondaryAxis(const FTransform& OriginalTransform, const FVector& PrimaryAxis, const FVector& SecondaryAxis)
{
	FTransform Transform = OriginalTransform;
	if (!PrimaryAxis.IsNearlyZero())
	{
		FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis);
		FVector Target1 = Transform.GetRotation().GetForwardVector();
		FQuat Rotation1 = FQuat::FindBetweenNormals(Axis, Target1);
		Transform.SetRotation((Rotation1 * Transform.GetRotation()).GetNormalized());
	}

	if (!SecondaryAxis.IsNearlyZero())
	{
		FVector Axis = Transform.TransformVectorNoScale(SecondaryAxis);
		FVector Target1 = Transform.GetRotation().GetUpVector();
		FQuat Rotation1 = FQuat::FindBetweenNormals(Axis, Target1);
		Transform.SetRotation((Rotation1 * Transform.GetRotation()).GetNormalized());
	}

	return Transform;
}

FRigUnit_ControlRigSplineFromPoints_Execute()
{
	// reset the spline
	Spline = FControlRigSpline();

	if (Points.Num() < 4)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Cannot create a spline with less than 4 points (%d received)."), Points.Num());
		return;
	}
		
	const TArrayView<const FVector> PointsView(Points.GetData(), Points.Num());
	Spline.SetControlPoints(PointsView, SplineMode, bClosed, SamplesPerSegment, Compression, Stretch);
}

FRigUnit_ControlRigSplineFromTransforms_Execute()
{
	// reset the spline
	Spline = FControlRigSpline();

	if (Transforms.Num() < 4)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Cannot create a spline with less than 4 transforms (%d received)."), Transforms.Num());
		return;
	}
		
	const TArrayView<const FTransform> TransformsView(Transforms.GetData(), Transforms.Num());
	Spline.SetControlTransforms(TransformsView, SplineMode, bClosed, SamplesPerSegment, Compression, Stretch);
}

FRigUnit_SetSplinePoints_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline.");
		return;
	}
	
	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	if (!Spline.SplineData->bClosed)
	{
		if (Points.Num() != Spline.SplineData->GetControlTransforms().Num())
		{
			UE_LOGF(LogControlRig, Error, "Number of input points does not match the number of point in the spline.");
			return;
		}
	}
	else if(Points.Num() != Spline.SplineData->GetControlTransforms().Num() - Spline.GetDegree())
	{
		if (Points.Num() != Spline.SplineData->GetControlTransforms().Num())
		{
			UE_LOGF(LogControlRig, Error, "Number of input points does not match the number of point in the spline.");
			return;
		}
	}

	const TArrayView<const FVector> PointsView(Points.GetData(), Points.Num());
	Spline.SetControlPoints(PointsView, Spline.SplineData->SplineMode, Spline.SplineData->bClosed, Spline.SplineData->SamplesPerSegment, Spline.SplineData->Compression, Spline.SplineData->Stretch);
}

FRigUnit_SetSplineTransforms_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline.");
		return;
	}
	
	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	if (!Spline.SplineData->bClosed)
	{
		if (Transforms.Num() != Spline.SplineData->GetControlTransforms().Num())
		{
			UE_LOGF(LogControlRig, Error, "Number of input points does not match the number of point in the spline.");
			return;
		}
	}
	else if(Transforms.Num() != Spline.SplineData->GetControlTransforms().Num() - Spline.GetDegree())
	{
		if (Transforms.Num() != Spline.SplineData->GetControlTransforms().Num())
		{
			UE_LOGF(LogControlRig, Error, "Number of input points does not match the number of point in the spline.");
			return;
		}
	}

	const TArrayView<const FTransform> TransformsView(Transforms.GetData(), Transforms.Num());
	Spline.SetControlTransforms(TransformsView, Spline.SplineData->SplineMode, Spline.SplineData->bClosed, Spline.SplineData->SamplesPerSegment, Spline.SplineData->Compression, Spline.SplineData->Stretch);
}

FRigUnit_PositionFromControlRigSpline_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	Position = Spline.PositionAtParam(U);
}

FRigUnit_TransformFromControlRigSpline_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	FVector UpVectorNormalized = UpVector;
	UpVectorNormalized.Normalize();

	const float ClampedU = FMath::Clamp<float>(U, 0.f, 1.f);
	const float ClampedRoll = FMath::Clamp<float>(Roll, -180.f, 180.f);

	FVector Tangent = Spline.TangentAtParam(ClampedU);

	// Check if Tangent can be normalized. If not, keep the same tangent as before.
	if (!Tangent.Normalize())
	{
		Tangent = Transform.ToMatrixNoScale().GetUnitAxis(EAxis::X);
	}
	FVector Binormal = FVector::CrossProduct(Tangent, UpVectorNormalized);
	Binormal = Binormal.RotateAngleAxis(ClampedRoll * ClampedU, Tangent);
	
	FMatrix RotationMatrix = FRotationMatrix::MakeFromXZ(Tangent, Binormal);

	Transform.SetFromMatrix(RotationMatrix);
	Transform.SetTranslation(Spline.PositionAtParam(U));
}

FRigUnit_TransformFromControlRigSpline2_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	const float ClampedU = FMath::Clamp<float>(U, 0.f, 1.f);
	Transform = Spline.TransformAtParam(ClampedU);

	Transform = ApplyPrimarySecondaryAxis(Transform, PrimaryAxis, SecondaryAxis);
}

FRigUnit_TangentFromControlRigSpline_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	const float ClampedU = FMath::Clamp<float>(U, 0.f, 1.f);

	FVector NewTangent = Spline.TangentAtParam(ClampedU);

	// Check if Tangent can be normalized. If not, keep the same tangent as before.
	if (!NewTangent.Normalize())
	{
		NewTangent = Tangent;
	}
	else
	{
		Tangent = NewTangent;
	}
}

FRigUnit_DrawControlRigSpline_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr)
	{
		return;
	}
	
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	int32 Count = FMath::Clamp<int32>(Detail, 4, 64);
	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::LineStrip, Color, Thickness);
	Instruction.Positions.SetNumUninitialized(Count);

	float T = 0;
	float Step = 1.f / float(Count-1);
	for(int32 Index=0; Index<Count; ++Index)
	{
		// Evaluate at T
		Instruction.Positions[Index] = Spline.PositionAtParam(T);
		T += Step;
	}

	ExecuteContext.GetDrawInterface()->Instructions.Add(Instruction);
}

FRigUnit_GetLengthControlRigSpline_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		Length = 0;
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		Length = 0;
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	Length = Spline.SplineData->AccumulatedLenth.Last();
}

FRigUnit_GetLengthAtParamControlRigSpline_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	Length = Spline.LengthAtParam(U);
}

FRigUnit_FitChainToSplineCurve_Execute()
{
	FRigUnit_FitChainToSplineCurveItemArray::StaticExecute(
		ExecuteContext,
		Items.Keys,
		Spline,
		Alignment,
		Minimum,
		Maximum,
		SamplingPrecision,
		PrimaryAxis,
		SecondaryAxis,
		PoleVectorPosition,
		Rotations,
		RotationEaseType,
		Weight,
		bPropagateToChildren,
		DebugSettings,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_FitChainToSplineCurve::GetUpgradeInfo() const
{
	FRigUnit_FitChainToSplineCurveItemArray NewNode;
	NewNode.Items = Items.GetKeys();
	NewNode.Spline = Spline;
	NewNode.Alignment = Alignment;
	NewNode.Minimum = Minimum;
	NewNode.Maximum = Maximum;
	NewNode.SamplingPrecision = SamplingPrecision;
	NewNode.PrimaryAxis = PrimaryAxis;
	NewNode.SecondaryAxis = SecondaryAxis;
	NewNode.PoleVectorPosition = PoleVectorPosition;
	NewNode.Rotations = Rotations;
	NewNode.RotationEaseType = RotationEaseType;
	NewNode.Weight = Weight;
	NewNode.bPropagateToChildren = bPropagateToChildren;
	NewNode.DebugSettings = DebugSettings;
	NewNode.WorkData = WorkData;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_FitSplineCurveToChain_Execute()
{
	FRigUnit_FitSplineCurveToChainItemArray::StaticExecute(ExecuteContext, Items.Keys, Spline);
}

FRigUnit_FitChainToSplineCurveItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	float& ChainLength = WorkData.ChainLength;
	TArray<FVector>& ItemPositions = WorkData.ItemPositions;
	TArray<float>& ItemSegments = WorkData.ItemSegments;
	TArray<FVector>& CurvePositions = WorkData.CurvePositions;
	TArray<float>& CurveSegments = WorkData.CurveSegments;
	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	TArray<int32>& ItemRotationA = WorkData.ItemRotationA;
	TArray<int32>& ItemRotationB = WorkData.ItemRotationB;
	TArray<float>& ItemRotationT = WorkData.ItemRotationT;
	TArray<FTransform>& ItemLocalTransforms = WorkData.ItemLocalTransforms;

	if(CachedItems.Num() == 0 && Items.Num() > 1)
	{
		CurvePositions.Reset();
		ItemPositions.Reset();
		ItemSegments.Reset();
		CurveSegments.Reset();
		ItemRotationA.Reset();
		ItemRotationB.Reset();
		ItemRotationT.Reset();
		ItemLocalTransforms.Reset();
		ChainLength = 0.f;

		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		if (CachedItems.Num() < 2)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough bones. You need at least two in the chain!"));
			return;
		}

		ItemPositions.SetNumZeroed(CachedItems.Num());
		ItemSegments.SetNumZeroed(CachedItems.Num());
		ItemSegments[0] = 0;
		for (int32 Index = 1; Index < CachedItems.Num(); Index++)
		{
			FVector A = Hierarchy->GetGlobalTransform(CachedItems[Index - 1]).GetLocation();
			FVector B = Hierarchy->GetGlobalTransform(CachedItems[Index]).GetLocation();
			ItemSegments[Index] = (A - B).Size();
			ChainLength += ItemSegments[Index];
		}

		ItemRotationA.SetNumZeroed(CachedItems.Num());
		ItemRotationB.SetNumZeroed(CachedItems.Num());
		ItemRotationT.SetNumZeroed(CachedItems.Num());
		ItemLocalTransforms.SetNumZeroed(CachedItems.Num());

		if (Rotations.Num() > 1)
		{
			TArray<float> RotationRatios;
			TArray<int32> RotationIndices;

			for (const FRigUnit_FitChainToCurve_Rotation& Rotation : Rotations)
			{
				RotationIndices.Add(RotationIndices.Num());
				RotationRatios.Add(FMath::Clamp<float>(Rotation.Ratio, 0.f, 1.f));
			}

			TLess<> Predicate;
			auto Projection = [RotationRatios](int32 Val) -> float
			{
				return RotationRatios[Val];
			};
			Algo::SortBy(RotationIndices, Projection, Predicate);

			for (int32 Index = 0; Index < CachedItems.Num(); Index++)
			{
				float T = 0.f;
				if (CachedItems.Num() > 1)
				{
					T = float(Index) / float(CachedItems.Num() - 1);
				}

				if (T <= RotationRatios[RotationIndices[0]])
				{
					ItemRotationA[Index] = ItemRotationB[Index] = RotationIndices[0];
					ItemRotationT[Index] = 0.f;
				}
				else if (T >= RotationRatios[RotationIndices.Last()])
				{
					ItemRotationA[Index] = ItemRotationB[Index] = RotationIndices.Last();
					ItemRotationT[Index] = 0.f;
				}
				else
				{
					for (int32 RotationIndex = 1; RotationIndex < RotationIndices.Num(); RotationIndex++)
					{
						int32 A = RotationIndices[RotationIndex - 1];
						int32 B = RotationIndices[RotationIndex];

						if (FMath::IsNearlyEqual(Rotations[A].Ratio, T))
						{
							ItemRotationA[Index] = ItemRotationB[Index] = A;
							ItemRotationT[Index] = 0.f;
							break;
						}
						else if (FMath::IsNearlyEqual(Rotations[B].Ratio, T))
						{
							ItemRotationA[Index] = ItemRotationB[Index] = B;
							ItemRotationT[Index] = 0.f;
							break;
						}
						else if (Rotations[B].Ratio > T)
						{
							if (FMath::IsNearlyEqual(RotationRatios[A], RotationRatios[B]))
							{
								ItemRotationA[Index] = ItemRotationB[Index] = A;
								ItemRotationT[Index] = 0.f;
							}
							else
							{
								ItemRotationA[Index] = A;
								ItemRotationB[Index] = B;
								ItemRotationT[Index] = (T - RotationRatios[A]) / (RotationRatios[B] - RotationRatios[A]);
								ItemRotationT[Index] = FControlRigMathLibrary::EaseFloat(ItemRotationT[Index], RotationEaseType);
							}
							break;
						}
					}
				}
			}
 		}
	}

	if (CachedItems.Num() < 2)
	{
		return;
	}

	int32 Samples = FMath::Clamp<int32>(SamplingPrecision, 4, 64);
	if (CurvePositions.Num() != Samples)
	{
		CurvePositions.SetNum(Samples + 1);
		CurveSegments.SetNum(Samples + 1);
	}

	FVector StartTangent = FVector::ZeroVector;
	FVector EndTangent = FVector::ZeroVector;

	float CurveLength = 0.f;
	for (int32 SampleIndex = 0; SampleIndex < Samples; SampleIndex++)
	{
		float T = float(SampleIndex) / float(Samples - 1);
		T = FMath::Lerp<float>(Minimum, Maximum, T);

		CurvePositions[SampleIndex] = Spline.PositionAtParam(T);
		FVector Tangent = Spline.TangentAtParam(T);
		if (SampleIndex == 0)
		{
			StartTangent = Tangent;
		}
		else if (SampleIndex == Samples - 1)
		{
			EndTangent = Tangent;
		}

		if (SampleIndex > 0)
		{
			CurveSegments[SampleIndex] = (CurvePositions[SampleIndex] - CurvePositions[SampleIndex - 1]).Size();
			CurveLength += CurveSegments[SampleIndex];
		}
		else
		{
			CurveSegments[SampleIndex] = 0.f;
		}
	}

	CurvePositions[Samples] = CurvePositions[Samples - 1] + EndTangent * ChainLength;
	CurveSegments[Samples] = ChainLength;

	if (ChainLength < SMALL_NUMBER)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("The chain has no length - all of the bones are in the sample place!"));
		return;
	}

	if (CurveLength < SMALL_NUMBER)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("The curve has no length - all of the points are in the sample place!"));
		return;
	}

	int32 CurveIndex = 1;
	ItemPositions[0] = CurvePositions[0];

	for (int32 Index = 1; Index < CachedItems.Num(); Index++)
	{
		const FVector& LastPosition = ItemPositions[Index - 1];

		float BoneLength = ItemSegments[Index];
		switch (Alignment)
		{
			case EControlRigCurveAlignment::Front:
			{
				break;
			}
			case EControlRigCurveAlignment::Stretched:
			{
				BoneLength = BoneLength * CurveLength / ChainLength;
				break;
			}
		}

		FVector A = FVector::ZeroVector;
		FVector B = FVector::ZeroVector;
		A = CurvePositions[CurveIndex - 1];
		B = CurvePositions[CurveIndex];

		float DistanceA = (LastPosition - A).Size();
		float DistanceB = (LastPosition - B).Size();

		if (DistanceB > BoneLength)
		{
			float Ratio = BoneLength / DistanceB;
			ItemPositions[Index] = FMath::Lerp<FVector>(LastPosition, B, Ratio);
			continue;
		}

		while (CurveIndex < CurvePositions.Num() - 1)
		{
			CurveIndex++;
			A = B;
			B = CurvePositions[CurveIndex];
			DistanceA = DistanceB;
			DistanceB = (B - LastPosition).Size();

			if ((DistanceA < BoneLength) != (DistanceB < BoneLength))
			{
				break;
			}
		}

		if (DistanceB < DistanceA)
		{
			FVector TempV = A;
			A = B;
			B = TempV;
			float TempF = DistanceA;
			DistanceA = DistanceB;
			DistanceB = TempF;
		}

		if (FMath::IsNearlyEqual(DistanceA, DistanceB))
		{
			ItemPositions[Index] = A;
			continue;
		}

		float Ratio = (BoneLength - DistanceA) / (DistanceB - DistanceA);
		ItemPositions[Index] = FMath::Lerp<FVector>(A, B, Ratio);
	}

	TArray<FTransform> BoneGlobalTransforms;
	if (Weight < 1.f - SMALL_NUMBER)
	{
		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
		{
			BoneGlobalTransforms.Add(Hierarchy->GetGlobalTransform(CachedItems[Index]));
		}
	}

	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		FTransform Transform = Hierarchy->GetGlobalTransform(CachedItems[Index]);

		Transform.SetTranslation(ItemPositions[Index]);

		FVector Target = FVector::ZeroVector;
		if (Index < CachedItems.Num() - 1)
		{
			Target = ItemPositions[Index + 1] - ItemPositions[Index];
		}
		else
		{
			Target = ItemPositions.Last() - ItemPositions[ItemPositions.Num() - 2];
		}

		if (!Target.IsNearlyZero() && !PrimaryAxis.IsNearlyZero())
		{
			Target = Target.GetSafeNormal();
			FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis).GetSafeNormal();
			FQuat Rotation = FQuat::FindBetweenNormals(Axis, Target);
			Transform.SetRotation((Rotation * Transform.GetRotation()).GetNormalized());
		}

		Target = PoleVectorPosition - ItemPositions[Index];
		if (!SecondaryAxis.IsNearlyZero())
		{
			if (!PrimaryAxis.IsNearlyZero())
			{
				FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis).GetSafeNormal();
				Target = Target - FVector::DotProduct(Target, Axis) * Axis;
			}

			if (!Target.IsNearlyZero() && !SecondaryAxis.IsNearlyZero())
			{
				Target = Target.GetSafeNormal();
				FVector Axis = Transform.TransformVectorNoScale(SecondaryAxis).GetSafeNormal();
				FQuat Rotation = FQuat::FindBetweenNormals(Axis, Target);
				Transform.SetRotation((Rotation * Transform.GetRotation()).GetNormalized());
			}
		}

		if (Weight >= 1.f - SMALL_NUMBER)
		{
			Hierarchy->SetGlobalTransform(CachedItems[Index], Transform, bPropagateToChildren && Rotations.Num() == 0);
		}
		else
		{
			Transform = FControlRigMathLibrary::LerpTransform(BoneGlobalTransforms[Index], Transform, FMath::Clamp<float>(Weight, 0.f, 1.f));
			Hierarchy->SetGlobalTransform(CachedItems[Index], Transform, bPropagateToChildren && Rotations.Num() == 0);
		}
	}

	if (Rotations.Num() > 0)
	{
		FTransform BaseTransform = FTransform::Identity;
		FRigElementKey ParentKey = Hierarchy->GetFirstParent(CachedItems[0].GetKey());
		if (ParentKey.IsValid())
		{
			BaseTransform = Hierarchy->GetGlobalTransform(ParentKey);
		}

		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
		{
			ItemLocalTransforms[Index] = Hierarchy->GetLocalTransform(CachedItems[Index]);
		}

		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
		{
			if (ItemRotationA[Index] >= Rotations.Num() ||
				ItemRotationB[Index] >= Rotations.Num())
			{
				continue;
			}

			FQuat Rotation = Rotations[ItemRotationA[Index]].Rotation;
			FQuat RotationB = Rotations[ItemRotationB[Index]].Rotation;
			if (ItemRotationA[Index] != ItemRotationB[Index])
			{
				if (ItemRotationT[Index] > 1.f - SMALL_NUMBER)
				{
					Rotation = RotationB;
				}
				else if (ItemRotationT[Index] > SMALL_NUMBER)
				{
					Rotation = FQuat::Slerp(Rotation, RotationB, ItemRotationT[Index]).GetNormalized();
				}
			}

			BaseTransform = ItemLocalTransforms[Index] * BaseTransform;
			Rotation = FQuat::Slerp(BaseTransform.GetRotation(), BaseTransform.GetRotation() * Rotation, FMath::Clamp<float>(Weight, 0.f, 1.f));
			BaseTransform.SetRotation(Rotation);
			Hierarchy->SetGlobalTransform(CachedItems[Index], BaseTransform, bPropagateToChildren);
		}
	}

	if (ExecuteContext.GetDrawInterface() != nullptr && DebugSettings.bEnabled)
	{
		int32 Count = 64;
		FRigVMDrawInstruction Instruction(ERigVMDrawSettings::LineStrip, DebugSettings.CurveColor, DebugSettings.Scale, DebugSettings.WorldOffset);
		Instruction.Positions.SetNumUninitialized(Count);

		float T = 0;
		float Step = 1.f / float(Count-1);
		for(int32 Index=0; Index<Count; ++Index)
		{
			// Evaluate at T
			Instruction.Positions[Index] = Spline.PositionAtParam(T);
			T += Step;
		}
		ExecuteContext.GetDrawInterface()->Instructions.Add(Instruction);

		for (auto Point : Spline.SplineData->GetControlPoints())
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(DebugSettings.WorldOffset, Point, DebugSettings.Scale * 6, DebugSettings.CurveColor);
		}

		ExecuteContext.GetDrawInterface()->DrawLineStrip(DebugSettings.WorldOffset, CurvePositions, DebugSettings.SegmentsColor, DebugSettings.Scale);
		ExecuteContext.GetDrawInterface()->DrawPoints(DebugSettings.WorldOffset, CurvePositions, DebugSettings.Scale * 4.f, DebugSettings.SegmentsColor);
	}
}

FRigUnit_SplineConstraint_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	const float ClampedMin = FMath::Min(FMath::Max(Minimum, 0.f), 1.f);
	float ClampedMax = FMath::Min(FMath::Max(Maximum, 0.f), 1.f);
	ClampedMax = FMath::Max(ClampedMax, ClampedMin);

	float& ChainLength = WorkData.ChainLength;
	TArray<FTransform>& ItemTransforms = WorkData.ItemTransforms;
	TArray<float>& ItemSegments = WorkData.ItemSegments;
	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;

	if(CachedItems.Num() == 0 && Items.Num() > 1)
	{
		ItemTransforms.Reset();
		ItemSegments.Reset();

		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		if (CachedItems.Num() < 2)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough bones. You need at least two in the chain!"));
			return;
		}

		ItemTransforms.SetNumZeroed(CachedItems.Num());
		ItemSegments.SetNumZeroed(CachedItems.Num());
		ChainLength = 0.f;
		ItemSegments[0] = 0.f;
		for (int32 Index = 1; Index < CachedItems.Num(); Index++)
		{
			FVector A = Hierarchy->GetGlobalTransform(CachedItems[Index - 1]).GetLocation();
			FVector B = Hierarchy->GetGlobalTransform(CachedItems[Index]).GetLocation();
			ItemSegments[Index] = (A - B).Size();
			ChainLength += ItemSegments[Index];
		}
	}

	if (CachedItems.Num() < 2)
	{
		return;
	}

	TArray<float> InitialLengths;
	if (Alignment == EControlRigCurveAlignment::Front)
	{
		ItemTransforms[0] = Spline.TransformAtParam(ClampedMin);
		float PrevParam = ClampedMin;
		float PrevLength = Spline.LengthAtParam(PrevParam);
		const float TotalSplineLength = Spline.LengthAtParam(1.f);
		const int32 LastIndex = Spline.SplineData->NumSamples() - 1;
		float AccumulatedLength = Spline.LengthAtParam(ClampedMin);
		for (int32 SegmentIndex = 1; SegmentIndex < ItemSegments.Num(); ++SegmentIndex)
		{
			const float& SegmentLenth = ItemSegments[SegmentIndex];
			AccumulatedLength += SegmentLenth;
			FTransform& Transform = ItemTransforms[SegmentIndex];
			bool bFoundSample = false;
			if (AccumulatedLength < TotalSplineLength)
			{
				const float fStartSampleSearch = PrevParam * LastIndex;	
				const int32 StartSampleSearch = FMath::Floor<int32>(fStartSampleSearch);

				// Find the sample interval where the bone should be
				for(int32 SampleIndex = StartSampleSearch; SampleIndex <= LastIndex; SampleIndex++)
				{
					const float CurrentSegmentLength = Spline.SplineData->AccumulatedLenth[SampleIndex];
					if (CurrentSegmentLength > AccumulatedLength)
					{
						const int32 IndexPrev = SampleIndex-1;
						const int32 IndexNext = SampleIndex;
						const float& IndexPrevLength = Spline.SplineData->AccumulatedLenth[IndexPrev];
						const float& IndexNextLength = Spline.SplineData->AccumulatedLenth[IndexNext];
						const float ULocal = (AccumulatedLength - IndexPrevLength) / (IndexNextLength - IndexPrevLength);
						check(ULocal >= 0.f && ULocal <= 1.f);
						const float UPrev = IndexPrev / (float)LastIndex;
						const float UNext = IndexNext / (float)LastIndex;
						PrevParam = UPrev + (ULocal * (UNext - UPrev));
						check(PrevParam >= ClampedMin && PrevParam <= 1.f);
						PrevLength = (ULocal-1.f) * IndexPrevLength + ULocal * IndexNextLength;

						Transform = Spline.TransformAtParam(PrevParam);
						bFoundSample = true;
						break;
					}
				}
			}
			if (!bFoundSample)
			{
				Transform = ItemTransforms.IsValidIndex(SegmentIndex-1) ? ItemTransforms[SegmentIndex-1] : FTransform::Identity;
				Transform.SetLocation(Transform.GetLocation() + SegmentLenth*Transform.GetRotation().GetForwardVector());
			}
			check(Transform.GetRotation().IsNormalized());
		}
	}
	else if (Alignment == EControlRigCurveAlignment::Stretched)
	{
		ItemTransforms[0] = Spline.TransformAtParam(ClampedMin);
		ItemTransforms.Last() = Spline.TransformAtParam(ClampedMax);

		float AccumulatedLength = 0.f;
		for (int32 SegmentIndex=0; SegmentIndex < ItemSegments.Num(); ++SegmentIndex)
		{
			AccumulatedLength += ItemSegments[SegmentIndex];
			const float U = AccumulatedLength*ClampedMax/ChainLength;
			const float LocalParam = (1.f - U)*ClampedMin + U*ClampedMax;
			ItemTransforms[SegmentIndex] = Spline.TransformAtParam(LocalParam);
		}
	}

	for (int32 Index = 0; Index < ItemTransforms.Num(); Index++)
	{
		FTransform& Transform = ItemTransforms[Index];
		Transform = ApplyPrimarySecondaryAxis(Transform, PrimaryAxis, SecondaryAxis);
	}
	
	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		Hierarchy->SetGlobalTransform(CachedItems[Index], ItemTransforms[Index], bPropagateToChildren);
	}
}

FRigVMStructUpgradeInfo FRigUnit_FitSplineCurveToChain::GetUpgradeInfo() const
{
	FRigUnit_FitSplineCurveToChainItemArray NewNode;
	NewNode.Items = Items.GetKeys();
	NewNode.Spline = Spline;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_FitSplineCurveToChainItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}

	if (Items.Num() < 4)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Cannot create a spline with less than 4 points (%d received)."), Items.Num());
		return;
	}

	const bool bClosed = Spline.SplineData->bClosed;

	// 1.- Create spline from chain
	TArray<FTransform> AuxControlTransforms;
	AuxControlTransforms.SetNumUninitialized(Items.Num());	
	for (int32 i = 0; i < Items.Num(); ++i)
	{
		const FRigElementKey& Key = Items[i];
		AuxControlTransforms[i] = Hierarchy->GetGlobalTransform(Key);
	}
    FControlRigSpline AuxSpline;
	const TArrayView<const FTransform> TransformsView(AuxControlTransforms.GetData(), AuxControlTransforms.Num());
	AuxSpline.SetControlTransforms(TransformsView, ESplineType::Hermite, bClosed, Spline.SplineData->SamplesPerSegment);
	

	// 2.-  for each control point in the original spline
	//			figure out its u in the original spline (preprocess)
	//			query position at u on the new spline
	const TArray<FTransform> ControlTransforms = Spline.SplineData->GetControlTransformsWithoutDuplicates();
	const int32 NumControlPoints = ControlTransforms.Num();
	TArray<FTransform> NewControlTransforms;
	NewControlTransforms.SetNumUninitialized(NumControlPoints);
	float U = 0;
	const float DeltaU = bClosed ? 1 / (float)(NumControlPoints) : 1 / (float)(NumControlPoints - 1);
	for (int32 i = 0; i < NumControlPoints; ++i)
	{
		FTransform NewPosition = AuxSpline.TransformAtParam(U);
		NewControlTransforms[i] = NewPosition;
		U += DeltaU;
	}

	const TArrayView<const FTransform> NewTransformsView(NewControlTransforms.GetData(), NewControlTransforms.Num());
	Spline.SetControlTransforms(NewTransformsView, Spline.SplineData->SplineMode, bClosed, Spline.SplineData->SamplesPerSegment, Spline.SplineData->Compression, Spline.SplineData->Stretch);
}

FRigUnit_ClosestParameterFromControlRigSpline_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	const int32 SampleCount = Spline.SplineData->SamplesArray.Num();

	auto DistanceToSegment = [&](int32 Index0, int32 Index1, float& OutParam) -> float
	{
		const int32 MaxIndex = Spline.SplineData->SamplesArray.Num()-1;
		const FVector& P0 = Spline.SplineData->SamplesArray[Index0].GetTranslation();
		const FVector& P1 = Spline.SplineData->SamplesArray[Index1].GetTranslation();

		const FVector V = P1 - P0;
		const FVector U = P0 - Position;
		const float T = - U.Dot(V) / V.Dot(V);
		if (T > 0 && T < 1)
		{
			OutParam = (1-T)*(Index0/(float)MaxIndex) + T*(Index1/(float)MaxIndex);
			return U.Cross(V).Length() / V.Length();
		}

		const float D0 = FVector::Distance(P0, Position);
		const float D1 = FVector::Distance(P1, Position);
		if (D0 < D1)
		{
			OutParam = Index0/(float)MaxIndex;
			return D0;
		}

		OutParam = Index1/(float)MaxIndex;
		return D1;
	};

	float ClosestDistance = TNumericLimits<float>::Max();
	for (int32 i=1; i<SampleCount; ++i)
	{
		float Param;
		float Distance = DistanceToSegment(i-1, i, Param);

		if (Distance < ClosestDistance)
		{
			ClosestDistance = Distance;
			U = Param;
		}
	}
}

FRigUnit_ParameterAtPercentage_Execute()
{
	if (!Spline.SplineData.IsValid())
	{
		return;
	}

	if (Spline.SplineData->Spline == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Invalid input spline implementation.");
		return;
	}
	
	const int32 SampleCount = Spline.SplineData->SamplesArray.Num();

	float Length = Spline.SplineData->AccumulatedLenth.Last();
	float ClampedPercentage = FMath::Clamp(Percentage, 0.f, 1.f);

	float SearchLength = Length * ClampedPercentage;
	int32 NextIndex = Algo::LowerBound(Spline.SplineData->AccumulatedLenth, SearchLength);

	if (NextIndex >= SampleCount)
	{
		U = 1.f;
		return;
	}

	if (NextIndex <= 0)
	{
		U = 0.f;
		return;
	}

	float UPrev = (NextIndex - 1) / (float) (SampleCount - 1);
	float UNext = NextIndex / (float) (SampleCount - 1);

	float LengthPrev = Spline.SplineData->AccumulatedLenth[NextIndex-1];
	float LengthNext = Spline.SplineData->AccumulatedLenth[NextIndex];

	float Interp = (SearchLength - LengthPrev) / (LengthNext - LengthPrev);
	U = (1 - Interp) * UPrev + Interp * UNext;
}


