// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigDefines.h"
#include "ControlRigSplineTypes.generated.h"

#define UE_API CONTROLRIGSPLINE_API

class UControlRig;

UENUM()
enum class ESplineType : uint8
{
	/** BSpline */
	/** The smooth curve will pass through the first and last control points */
	BSpline,

	/** Hermite: */
	/** The curve will pass through the control points */
	Hermite,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

// Reading material
// 
// https://pages.mtu.edu/~shene/COURSES/cs3621/NOTES/notes.html
// https://github.com/caadxyz/DeBoorAlgorithmNurbs
// https://stackoverflow.com/questions/32897943/how-do-catmull-rom-and-hermite-splines-relate

// https://stackoverflow.com/questions/30748316/%20catmull-rom-interpolation-on-svg-paths/30826434#30826434
// https://stackoverflow.com/questions/32897943/how-do-catmull-rom-and-hermite-splines-relate
// https://www.cs.cmu.edu/~fp/courses/graphics/asst5/catmullRom.pdf

class ControlRigBaseSpline
{

protected:
	TArray<FVector> ControlPoints;
	uint16 Degree;
	bool bClosed;
	
public:

	ControlRigBaseSpline(const TArrayView<const FVector>& InControlPoints, uint16 InDegree, bool bInClosed);
	virtual ~ControlRigBaseSpline()= default;
	
	virtual FVector GetPointAtParam(float Param) = 0;

	virtual void SetControlPoints(const TArrayView<const FVector>& InControlPoints);

	virtual TArray<FVector>& GetControlPoints() { return ControlPoints; }
	
	virtual TArray<FVector> GetControlPointsWithoutDuplicates();

	virtual TArray<uint16> GetControlIndicesWithoutDuplicates();

	uint8 GetDegree() const { return Degree; }
};

class ControlRigBSpline : public ControlRigBaseSpline
{
	TArray<float> KnotVector;
	
public:
	ControlRigBSpline(const TArrayView<const FVector>& InControlPoints, const uint16 InDegree, const bool bInClosed, const bool bInClamped);

	virtual FVector GetPointAtParam(float Param);

};

class ControlRigHermite : public ControlRigBaseSpline
{
	TArray<FVector> SegmentPoints;

	uint16 NumSegments;
	
public:
	ControlRigHermite(const TArrayView<const FVector>& InControlPoints, const bool bInClosed);

	virtual void SetControlPoints(const TArrayView<const FVector>& InControlPoints) override;
	
	virtual FVector GetPointAtParam(float Param);

};


USTRUCT()
struct FControlRigSplineImpl
{
	GENERATED_BODY()

	FControlRigSplineImpl()	
	{
		SplineMode = ESplineType::BSpline;
		Spline = nullptr;
		bClosed = false;
		SamplesPerSegment = 16;
	}

	UE_API virtual ~FControlRigSplineImpl();

	// Spline type
	ESplineType SplineMode;

	// The transforms of the control points
	TArray<FTransform> ControlTransforms;

	// The initial lengths between samples
	TArray<float> InitialLengths;

	// The actual spline
	ControlRigBaseSpline* Spline;

	// Wether or not the curve is closed
	bool bClosed;

	// Samples per segment, where segment is the portion between two control points
	int32 SamplesPerSegment;

	// The allowed length compression (1.f being do not allow compression`). If 0, no restriction wil be applied.
	float Compression;

	// The allowed length stretch (1.f being do not allow stretch`). If 0, no restriction wil be applied.
	float Stretch;

	// Positions along the "real" curve (no samples in the first and last segments of a hermite spline)
	TArray<FTransform> SamplesArray;

	// Accumulated length along the spline given by samples
	TArray<float> AccumulatedLenth;

	// Returns a reference to the control points that were used to create this spline 
	UE_API TArray<FVector>& GetControlPoints();

	// Returns a reference to the control transforms that were used to create this spline 
	UE_API TArray<FTransform>& GetControlTransforms();

	// Returns the control points that were used to create this spline, removing the duplicates in case of a closed spline
	UE_API TArray<FVector> GetControlPointsWithoutDuplicates();
	
	// Returns the control transforms that were used to create this spline, removing the duplicates in case of a closed spline
	UE_API TArray<FTransform> GetControlTransformsWithoutDuplicates();

	// Returns the degree of the curve
	UE_API uint8 GetDegree() const;

	// Returns the total number of samples cached
	UE_API uint16 NumSamples() const;
};

USTRUCT(BlueprintType)
struct FControlRigSpline 
{
	GENERATED_BODY()

	FControlRigSpline()	{}

	virtual ~FControlRigSpline() = default;

	UE_API FControlRigSpline(const FControlRigSpline& InOther);
	UE_API FControlRigSpline& operator =(const FControlRigSpline& InOther);

	TSharedPtr<FControlRigSplineImpl> SplineData;

	/**
	* Returns the degree of the spline
	*
	* @return			The degree of the spline.
	*/
	UE_API uint8 GetDegree() const;

	/**
	* Sets the control points in the spline. It will build the spline if needed, 
	* or will update the points if building from scratch is not necessary. The type of spline to build will
	* depend on what is set in SplineMode.
	*
	* @param InPoints	The control points to set.
	* @param SplineMode	The type of spline
	* @param bInClosed	If the spline should be closed
	* @param SamplesPerSegment The samples to cache for every segment defined between two control rig
	* @param Compression The allowed length compression (1.f being do not allow compression`). If 0, no restriction wil be applied.
	* @param Stretch The allowed length stretch (1.f being do not allow stretch). If 0, no restriction wil be applied.
	*/
	UE_API void SetControlPoints(const TArrayView<const FVector>& InPoints, const ESplineType SplineMode = ESplineType::BSpline, const bool bInClosed = false, const int32 SamplesPerSegment = 16, const float Compression = 1.f, const float Stretch = 1.f);

	/**
	* Sets the control transforms in the spline. It will build the spline if needed, 
	* or will update the transforms if building from scratch is not necessary. The type of spline to build will
	* depend on what is set in SplineMode.
	*
	* @param InTransforms	The control transforms to set.
	* @param SplineMode	The type of spline
	* @param bInClosed	If the spline should be closed
	* @param SamplesPerSegment The samples to cache for every segment defined between two control rig
	* @param Compression The allowed length compression (1.f being do not allow compression`). If 0, no restriction wil be applied.
	* @param Stretch The allowed length stretch (1.f being do not allow stretch). If 0, no restriction wil be applied.
	*/
	UE_API void SetControlTransforms(const TArrayView<const FTransform>& InTransforms, const ESplineType SplineMode = ESplineType::BSpline, const bool bInClosed = false, const int32 SamplesPerSegment = 16, const float Compression = 1.f, const float Stretch = 1.f);

	/**
	* Given an InParam float in [0, 1], will return the position of the spline at that point.
	*
	* @param InParam	The parameter between [0, 1] to query.
	* @return			The position in the spline.
	*/
	UE_API FVector PositionAtParam(const float InParam) const;
	
	/**
	* Given an InParam float in [0, 1], will return the transform of the spline at that point.
	*
	* @param InParam	The parameter between [0, 1] to query.
	* @return			The transform in the spline.
	*/
	UE_API FTransform TransformAtParam(const float InParam) const;

	/**
	* Given an InParam float in [0, 1], will return the tangent vector of the spline at that point. 
	* Note that this vector is not normalized.
	*
	* @param InParam	The parameter between [0, 1] to query.
	* @return			The tangent of the spline at InParam.
	*/
	UE_API FVector TangentAtParam(const float InParam) const;

	/**
	* Given an InParam float in [0, 1], will return the length of the spline at that point. 
	*
	* @param InParam	The parameter between [0, 1] to query.
	* @return			The length of the spline at InParam.
	*/
	UE_API float LengthAtParam(const float InParam) const;
};

#undef UE_API
