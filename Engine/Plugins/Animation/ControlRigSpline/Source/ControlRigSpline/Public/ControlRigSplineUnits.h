// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "ControlRigDefines.h"
#include "Units/RigUnit.h"
#include "ControlRigSplineTypes.h"
#include "Units/Highlevel/Hierarchy/RigUnit_FitChainToCurve.h"
#include "ControlRigSplineUnits.generated.h"

#define UE_API CONTROLRIGSPLINE_API

/*
 * The base class for all control rig spline nodes
 */
USTRUCT(meta = (Abstract, Category = "Splines", Varying, NodeColor = "0.737911 0.099899 0.099899", DocumentationPolicy = "Strict"))
struct FRigUnit_ControlRigSplineBase : public FRigUnit
{
	GENERATED_BODY()
};

/*
 * The base class for all mutable control rig spline nodes
 */
USTRUCT(meta = (Abstract, Category = "Splines", Varying, NodeColor = "0.737911 0.099899 0.099899", DocumentationPolicy = "Strict"))
struct FRigUnit_ControlRigSplineBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

/*
 * Creates a Spline curve from an array of positions
 */
USTRUCT(meta = (DisplayName = "Spline From Points", Keywords="Spline From Positions"))
struct FRigUnit_ControlRigSplineFromPoints : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_ControlRigSplineFromPoints()
	{
		SplineMode = ESplineType::Hermite;
		bClosed = false;
		SamplesPerSegment = 16;
		Compression = 0.f;
		Stretch = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input points to form the spline
	UPROPERTY(meta = (Input))
	TArray<FVector> Points;

	// The mode to use for the spline
	UPROPERTY(meta = (Input))
	ESplineType SplineMode;

	// If True the spline will be closed 
	UPROPERTY(meta = (Input))
	bool bClosed;

	// Specifies the detail per segment of the spline
	UPROPERTY(meta = (Input, ClampMin = "1"))
	int32 SamplesPerSegment;

	// The amount of compression to apply
	UPROPERTY(meta = (Input))
	float Compression;

	// The amount of stretch to allow for the spline
	UPROPERTY(meta = (Input))
	float Stretch;

	// The resulting spline
	UPROPERTY(meta = (Output))
	FControlRigSpline Spline;
};

/*
 * Creates a Spline curve from an array of transforms
 */
USTRUCT(meta = (DisplayName = "Spline From Transforms", Keywords="Spline From Transforms"))
struct FRigUnit_ControlRigSplineFromTransforms : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_ControlRigSplineFromTransforms()
	{
		SplineMode = ESplineType::Hermite;
		bClosed = false;
		SamplesPerSegment = 16;
		Compression = 0.f;
		Stretch = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transforms to build the spline from
	UPROPERTY(meta = (Input))
	TArray<FTransform> Transforms;

	// The mode to use for the spline
	UPROPERTY(meta = (Input))
	ESplineType SplineMode;

	// If True the spline will be closed 
	UPROPERTY(meta = (Input))
	bool bClosed;

	// Specifies the detail per segment of the spline
	UPROPERTY(meta = (Input))
	int32 SamplesPerSegment;

	// The amount of compression to apply
	UPROPERTY(meta = (Input))
	float Compression;

	// The amount of stretch to allow for the spline
	UPROPERTY(meta = (Input))
	float Stretch;

	// The resulting spline
	UPROPERTY(meta = (Output))
	FControlRigSpline Spline;
};

/*
 * Set the points of a spline, given a spline and an array of positions
 */
USTRUCT(meta = (DisplayName = "Set Spline Points"))
struct FRigUnit_SetSplinePoints : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetSplinePoints()
	{
		
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The points to set on the spline. The assumption is that the
	// number of points provided matches the points on the spline.
	UPROPERTY(meta = (Input))
	TArray<FVector> Points;

	// The spline to be updated
	UPROPERTY(meta = (Input, Output))
	FControlRigSpline Spline;
};

/*
 * Set the points of a spline, given a spline and an array of transforms
 */
USTRUCT(meta = (DisplayName = "Set Spline Transforms"))
struct FRigUnit_SetSplineTransforms : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetSplineTransforms()
	{
		
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The transforms interpreted as positions to set on the spline. The assumption is that the
	// number of transforms provided matches the points on the spline.
	UPROPERTY(meta = (Input))
	TArray<FTransform> Transforms;

	// The spline to be updated
	UPROPERTY(meta = (Input, Output))
	FControlRigSpline Spline;
};

/*
 * Retrieves the position from a given Spline and U value
 */
USTRUCT(meta = (DisplayName = "Position From Spline", Keywords="Point From Spline"))
struct FRigUnit_PositionFromControlRigSpline : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_PositionFromControlRigSpline()
	{
		U = 0.f;
		Position = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The U value along the spline to evaluate
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float U;

	// The resulting position on the spline
	UPROPERTY(meta = (Output))
	FVector Position;
};

/*
 * Retrieves the transform from a given Spline and U value based on the given Up Vector and Roll
 */
USTRUCT(meta = (DisplayName = "Transform From Spline (with UpVector)"))
struct FRigUnit_TransformFromControlRigSpline : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_TransformFromControlRigSpline()
	{
		UpVector = FVector::UpVector;
		Roll = 0.f;
		U = 0.f;
		Transform = FTransform::Identity;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The up-vector to use to build the transform's rotation
	UPROPERTY(meta = (Input))
	FVector UpVector;

	// The roll to apply to the resulting rotation
	UPROPERTY(meta = (Input))
	float Roll;

	// The U value along the spline to evaluate
	UPROPERTY(meta = (Input))
	float U;

	// The resulting composed transform
	UPROPERTY(meta = (Output))
	FTransform Transform;
};

/*
 * Retrieves the transform from a given Spline and U value based on the given primary and secondary axis
 */
USTRUCT(meta = (DisplayName = "Transform From Spline"))
struct FRigUnit_TransformFromControlRigSpline2 : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_TransformFromControlRigSpline2()
	{
		U = 0.f;
		Transform = FTransform::Identity;
		PrimaryAxis = FVector::ZeroVector;
		SecondaryAxis = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The U value along the spline to evaluate
	UPROPERTY(meta = (Input))
	float U;

	// The primary axis to use when building the transform
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	// The secondary axis to use when building the transform 
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	// The resulting composed transform
	UPROPERTY(meta = (Output))
	FTransform Transform;
};

/*
 * Retrieves the tangent from a given Spline and U value
 */
USTRUCT(meta = (DisplayName = "Tangent From Spline"))
struct FRigUnit_TangentFromControlRigSpline : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_TangentFromControlRigSpline()
	{
		U = 0.f;
		Tangent = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The U value along the spline to evaluate
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float U;

	// The tangent at the evaluated location on the spline
	UPROPERTY(meta = (Output))
	FVector Tangent;
};

/*
 * Draws the given spline in the viewport
 */
USTRUCT(meta = (DisplayName = "Draw Spline"))
struct FRigUnit_DrawControlRigSpline : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DrawControlRigSpline()
	{
		Color = FLinearColor::Red;
		Thickness = 1.f;
		Detail = 16.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to draw
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use when drawing the spline
	UPROPERTY(meta = (Input))
	float Thickness;

	// The detail to use to render the spline
	UPROPERTY(meta = (Input))
	int32 Detail;
};

/*
 * Retrieves the length from a given Spline
 */
USTRUCT(meta = (DisplayName = "Get Length Of Spline"))
struct FRigUnit_GetLengthControlRigSpline : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_GetLengthControlRigSpline()
	{
		Length = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The resulting length of the spline
	UPROPERTY(meta = (Output))
	float Length;
};

/*
 * Retrieves the length from a given Spline
 */
USTRUCT(meta = (DisplayName = "Get Length At Param Of Spline"))
struct FRigUnit_GetLengthAtParamControlRigSpline : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_GetLengthAtParamControlRigSpline()
	{
		U = 0.f;
		Length = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The U value along the spline to evaluate
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float U;

	// The resulting length 
	UPROPERTY(meta = (Output))
	float Length;
};

/**
 * Fits a given chain to a spline curve.
 * Additionally provides rotational control matching the features of the Distribute Rotation node.
 */
USTRUCT(meta=(DisplayName="Fit Chain on Spline Curve", Category="Hierarchy", Keywords="Fit,Resample,Spline", Deprecated = "5.0"))
struct FRigUnit_FitChainToSplineCurve : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitChainToSplineCurve()
	{
		Alignment = EControlRigCurveAlignment::Stretched;
		Minimum = 0.f;
		Maximum = 1.f;
		SamplingPrecision = 12;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 0.f, 0.f);
		PoleVectorPosition = FVector::ZeroVector;
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_FitChainToCurve_DebugSettings();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/** 
	 * The curve to align to
	 */
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	/** 
	 * Specifies how to align the chain on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigCurveAlignment Alignment;

	/** 
	 * The minimum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Minimum;

	/** 
	 * The maximum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Maximum;

	/**
	 * The number of samples to use on the curve. Clamped at 64.
	 */
	UPROPERTY(meta = (Input, Constant))
	int32 SamplingPrecision;

	/**
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/**
	 * The minor axis being aligned - towards the pole vector.
	 * You can use (0.0, 0.0, 0.0) to disable it.
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * The position of the pole vector used for aligning the secondary axis.
	 * Only has an effect if the secondary axis is set.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVectorPosition;

	/** 
	 * The list of rotations to be applied along the curve
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_FitChainToCurve_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	// The debug settings to use
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_FitChainToCurve_DebugSettings DebugSettings;

	UPROPERTY(transient)
	FRigUnit_FitChainToCurve_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Fits a given chain to a spline curve.
 * Additionally provides rotational control matching the features of the Distribute Rotation node.
 */
USTRUCT(meta=(DisplayName="Fit Chain on Spline Curve", Keywords="Fit,Resample,Spline"))
struct FRigUnit_FitChainToSplineCurveItemArray : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitChainToSplineCurveItemArray()
	{
		Alignment = EControlRigCurveAlignment::Stretched;
		Minimum = 0.f;
		Maximum = 1.f;
		SamplingPrecision = 12;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 0.f, 0.f);
		PoleVectorPosition = FVector::ZeroVector;
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_FitChainToCurve_DebugSettings();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/** 
	 * The curve to align to
	 */
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	/** 
	 * Specifies how to align the chain on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigCurveAlignment Alignment;

	/** 
	 * The minimum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Minimum;

	/** 
	 * The maximum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Maximum;

	/**
	 * The number of samples to use on the curve. Clamped at 64.
	 */
	UPROPERTY(meta = (Input, Constant))
	int32 SamplingPrecision;

	/**
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/**
	 * The minor axis being aligned - towards the pole vector.
	 * You can use (0.0, 0.0, 0.0) to disable it.
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * The position of the pole vector used for aligning the secondary axis.
	 * Only has an effect if the secondary axis is set.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVectorPosition;

	/** 
	 * The list of rotations to be applied along the curve
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_FitChainToCurve_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	// The debug settings to use
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_FitChainToCurve_DebugSettings DebugSettings;

	UPROPERTY(transient)
	FRigUnit_FitChainToCurve_WorkData WorkData;
};

USTRUCT()
struct FRigUnit_SplineConstraint_WorkData
{
	GENERATED_BODY()

	FRigUnit_SplineConstraint_WorkData()
	{
		ChainLength = 0.f;
	}

	UPROPERTY()
	float ChainLength;

	UPROPERTY()
	TArray<FTransform> ItemTransforms;

	UPROPERTY()
	TArray<float> ItemSegments;

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;
};


/**
 * Fits a given chain to a spline curve.
 * Additionally provides rotational control matching the features of the Distribute Rotation node.
 */
USTRUCT(meta=(DisplayName="Spline Constraint", Keywords="Fit,Resample,Spline"))
struct FRigUnit_SplineConstraint : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SplineConstraint()
	{
		Alignment = EControlRigCurveAlignment::Stretched;
		Minimum = 0.f;
		Maximum = 1.f;
		bPropagateToChildren = true;
		PrimaryAxis = FVector::ZeroVector;
		SecondaryAxis = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/** 
	 * The curve to align to
	 */
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	/** 
	 * Specifies how to align the chain on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigCurveAlignment Alignment;

	/** 
	 * The minimum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float Minimum;

	/** 
	 * The maximum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float Maximum;

	// The primary axis to use when building transforms
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	// The secondary axis to use when building transforms
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_SplineConstraint_WorkData WorkData;
};


/**
 * Fits a given spline curve to a chain.
 */
USTRUCT(meta=(DisplayName="Fit Spline Curve on Chain", Category="Hierarchy", Keywords="Fit,Resample,Spline", Deprecated = "5.0"))
struct FRigUnit_FitSplineCurveToChain : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitSplineCurveToChain()
	{
		
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align to
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/** 
	 * The curve to align
	 */
	UPROPERTY(meta = (Input, Output))
	FControlRigSpline Spline;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Fits a given spline curve to a chain.
 */
USTRUCT(meta=(DisplayName="Fit Spline Curve on Chain", Keywords="Fit,Resample,Spline"))
struct FRigUnit_FitSplineCurveToChainItemArray : public FRigUnit_ControlRigSplineBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitSplineCurveToChainItemArray()
	{
		
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align to
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/** 
	 * The curve to align
	 */
	UPROPERTY(meta = (Input, Output))
	FControlRigSpline Spline;
};

/*
 * Retrieves the closest U value from a given Spline and a position
 */
USTRUCT(meta = (DisplayName = "Closest Parameter From Spline"))
struct FRigUnit_ClosestParameterFromControlRigSpline : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_ClosestParameterFromControlRigSpline()
	{
		Position = FVector::ZeroVector;
		U = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The position to evaluate
	UPROPERTY(meta = (Input))
	FVector Position;

	// The U value at the closest location on the spline
	UPROPERTY(meta = (Output))
	float U;
	
};

/*
 * Returns the U parameter of a spline given a length percentage (0.0 - 1.0)
 */
USTRUCT(meta = (DisplayName = "Parameter At Length Percentage"))
struct FRigUnit_ParameterAtPercentage : public FRigUnit_ControlRigSplineBase
{
	GENERATED_BODY()

	FRigUnit_ParameterAtPercentage()
	{
		Percentage = 0.f;
		U = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The spline to evaluate
	UPROPERTY(meta = (Input))
	FControlRigSpline Spline;

	// The percentage (0.0 - 1.0) to evaluate
	UPROPERTY(meta = (Input))
	float Percentage;

	// The U value on the spline for the given percentage
	UPROPERTY(meta = (Output))
	float U;
	
};

#undef UE_API
